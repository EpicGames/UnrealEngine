// Copyright Epic Games, Inc. All Rights Reserved.

#include "SunPositionModule.h"
#include "Modules/ModuleManager.h"
#include "SunPositionPlacement.h"
#include "SunPositionStyle.h"

void FSunPositionModule::StartupModule()
{
	FSunPositionStyle::Initialize();
	FSunPositionPlacement::RegisterPlacement();
}

void FSunPositionModule::ShutdownModule()
{
	FSunPositionStyle::Shutdown();
}

IMPLEMENT_MODULE(FSunPositionModule, SunPosition)
