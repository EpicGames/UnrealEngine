// Copyright Epic Games, Inc. All Rights Reserved.

#include "SunPositionPlacement.h"
#include "Engine/Blueprint.h"
#if WITH_EDITOR
#include "IPlacementModeModule.h"
#include "ActorFactories/ActorFactoryBlueprint.h"
#endif // WITH_EDITOR

void FSunPositionPlacement::RegisterPlacement()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		UBlueprint* SunSky = Cast<UBlueprint>(FSoftObjectPath(TEXT("/SunPosition/SunSky.SunSky")).TryLoad());
		if (SunSky == nullptr)
		{
			return;
		}

		IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
		FPlacementCategoryInfo Info = *PlacementModeModule.GetRegisteredPlacementCategory(FBuiltInPlacementCategories::Lights());

		FPlaceableItem* BPPlacement = new FPlaceableItem(
			*UActorFactoryBlueprint::StaticClass(),
			FAssetData(SunSky, true),
			FName("SunPosition.ModesThumbnail"),
			FName("SunPosition.ModesIcon"),
			TOptional<FLinearColor>(),
			TOptional<int32>(),
			NSLOCTEXT("PlacementMode", "Sun and Sky", "Sun and Sky")
		);

		IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable(BPPlacement));
	}
#endif // WITH_EDITOR
}
