// Copyright Epic Games, Inc. All Rights Reserved.

#include "SunPositionStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

TSharedPtr<FSlateStyleSet> FSunPositionStyle::StyleSet;

void FSunPositionStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(FName("SunPositionStyle"));

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FString IconPath = IPluginManager::Get().FindPlugin(TEXT("SunPosition"))->GetBaseDir() + TEXT("/Resources/SunPosition.png");
	StyleSet->Set("SunPosition.ModesThumbnail", new FSlateImageBrush(IconPath, FVector2D(40.0f, 40.0f)));
	StyleSet->Set("SunPosition.ModesIcon", new FSlateImageBrush(IconPath, FVector2D(16.0f, 16.0f)));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FSunPositionStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
