// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SunPosition.generated.h"


class UArrowComponent;

USTRUCT(BlueprintType)
struct SUNPOSITION_API FSunPositionData
{
	GENERATED_BODY()

public:

	FSunPositionData();

	/** Sun Elevation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sun Position")
	float Elevation;

	/** Sun Elevation, corrected for atmospheric diffraction */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sun Position")
	float CorrectedElevation;

	/** Sun azimuth */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sun Position")
	float Azimuth;

	/** Sunrise time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sun Position")
	FTimespan SunriseTime;

	/** Sunset time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sun Position")
	FTimespan SunsetTime;

	/** Solar noon */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sun Position")
	FTimespan SolarNoon;
};

UCLASS()
class SUNPOSITION_API USunPositionFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get the sun's position data based on position, date and time */
	UFUNCTION(BlueprintCallable, Category = "Sun Position")
	static void GetSunPosition(float Latitude, float Longitude, float TimeZone, bool bIsDaylightSavingTime, int32 Year, int32 Month, int32 Day, int32 Hours, int32 Minutes, int32 Seconds, FSunPositionData& SunPositionData);
};
