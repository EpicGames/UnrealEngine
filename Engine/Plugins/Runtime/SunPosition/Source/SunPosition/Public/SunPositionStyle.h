// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class FSunPositionStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static TSharedPtr<ISlateStyle> Get() { return StyleSet; }

private:
	static TSharedPtr<FSlateStyleSet> StyleSet;
};