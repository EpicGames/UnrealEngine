// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SunPosition : ModuleRules
	{
		public SunPosition(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
					"Projects",
                    "Slate",
					"SlateCore",
                }
            );

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"UnrealEd",
						"EditorFramework",
						"PlacementMode",
					}
				);
			}
		}
	}
}
