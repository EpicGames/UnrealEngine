{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Sun Position Calculator",
	"Description": "Calculates the sun position based on latitude/longitude and date/time.",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SunPosition",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit"
		}
	]
}